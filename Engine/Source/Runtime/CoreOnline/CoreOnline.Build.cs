// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class CoreOnline : ModuleRules
{
	public CoreOnline(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Json"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"CoreUObject"
			}
		);

		PrivateDefinitions.Add("COREONLINE_PACKAGE=1");
		PublicDefinitions.Add("PLATFORM_MAX_LOCAL_PLAYERS=" + GetPlatformMaxLocalPlayers(Target));
	}

	protected virtual int GetPlatformMaxLocalPlayers(ReadOnlyTargetRules Target)
	{
		// 0 indicates no platform override
		return 0;
	}
}
