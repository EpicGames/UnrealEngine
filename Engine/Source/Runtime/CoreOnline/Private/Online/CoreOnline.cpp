// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/CoreOnline.h"

#include "CoreGlobals.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/LazySingleton.h"
#include "Online/CoreOnlinePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CoreOnline)

#define UE_ONLINE_ONLINESERVICES_MAX_LENGTH 16

namespace UE::Online {

static const uint32 OnlineServicesAliasArraySize = 4;
TCHAR OnlineServicesAliasArray[OnlineServicesAliasArraySize][UE_ONLINE_ONLINESERVICES_MAX_LENGTH] = {};

void RegisterAliasForGameDefined(EOnlineServices Value, const TCHAR* InStr)
{
	uint32 Index = (uint8)Value - (uint8)EOnlineServices::GameDefined_0;
	check(Index >= 0 && Index < OnlineServicesAliasArraySize);
	check(FCString::Strlen(InStr) < UE_ONLINE_ONLINESERVICES_MAX_LENGTH);
	FCString::Strncpy(&OnlineServicesAliasArray[Index][0], InStr, UE_ONLINE_ONLINESERVICES_MAX_LENGTH);
}

const TCHAR* LexToString(EOnlineServices Value)
{
	if (Value >= EOnlineServices::GameDefined_0 && Value <= EOnlineServices((uint8)EOnlineServices::GameDefined_0 + OnlineServicesAliasArraySize))
	{
		uint32 Index = (uint8)Value - (uint8)EOnlineServices::GameDefined_0;
		check(Index >= 0 && Index < OnlineServicesAliasArraySize);
		const TCHAR* Alias = &OnlineServicesAliasArray[Index][0];
		if (Alias[0] != TEXT('\0'))
		{
			return Alias;
		}
	}

	switch (Value)
	{
	case EOnlineServices::Null:				return TEXT("Null");
	case EOnlineServices::Epic:				return TEXT("Epic");
	case EOnlineServices::Xbox:				return TEXT("Xbox");
	case EOnlineServices::PSN:				return TEXT("PSN");
	case EOnlineServices::Nintendo:			return TEXT("Nintendo");
	case EOnlineServices::Reserved_5:		return TEXT("Reserved_5");
	case EOnlineServices::Steam:			return TEXT("Steam");
	case EOnlineServices::Google:			return TEXT("Google");
	case EOnlineServices::GooglePlay:		return TEXT("GooglePlay");
	case EOnlineServices::Apple:			return TEXT("Apple");
	case EOnlineServices::AppleGameKit:		return TEXT("AppleGameKit");
	case EOnlineServices::Samsung:			return TEXT("Samsung");
	case EOnlineServices::Oculus:			return TEXT("Oculus");
	case EOnlineServices::Tencent:			return TEXT("Tencent");
	case EOnlineServices::EpicAccount:		return TEXT("EpicAccount");
	case EOnlineServices::EpicGame:			return TEXT("EpicGame");
	case EOnlineServices::Reserved_16:		return TEXT("Reserved_16");
	case EOnlineServices::Reserved_17:		return TEXT("Reserved_17");
	case EOnlineServices::Reserved_18:		return TEXT("Reserved_18");
	case EOnlineServices::Reserved_19:		return TEXT("Reserved_19");
	case EOnlineServices::Reserved_20:		return TEXT("Reserved_20");
	case EOnlineServices::Reserved_21:		return TEXT("Reserved_21");
	case EOnlineServices::Reserved_22:		return TEXT("Reserved_22");
	case EOnlineServices::Reserved_23:		return TEXT("Reserved_23");
	case EOnlineServices::Reserved_24:		return TEXT("Reserved_24");
	case EOnlineServices::Reserved_25:		return TEXT("Reserved_25");
	case EOnlineServices::Reserved_26:		return TEXT("Reserved_26");
	case EOnlineServices::Reserved_27:		return TEXT("Reserved_27");
	case EOnlineServices::GameDefined_0:	return TEXT("GameDefined_0");
	case EOnlineServices::GameDefined_1:	return TEXT("GameDefined_1");
	case EOnlineServices::GameDefined_2:	return TEXT("GameDefined_2");
	case EOnlineServices::GameDefined_3:	return TEXT("GameDefined_3");
	default:								checkNoEntry(); // Intentional fallthrough
	case EOnlineServices::None:				return TEXT("None");
	case EOnlineServices::Platform:			return TEXT("Platform");
	case EOnlineServices::Default:			return TEXT("Default");
	}
}

void LexFromString(EOnlineServices& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Null")) == 0)
	{
		OutValue = EOnlineServices::Null;
	}
	else if (FCString::Stricmp(InStr, TEXT("Epic")) == 0)
	{
		OutValue = EOnlineServices::Epic;
	}
	else if (FCString::Stricmp(InStr, TEXT("Xbox")) == 0)
	{
		OutValue = EOnlineServices::Xbox;
	}
	else if (FCString::Stricmp(InStr, TEXT("PSN")) == 0)
	{
		OutValue = EOnlineServices::PSN;
	}
	else if (FCString::Stricmp(InStr, TEXT("Nintendo")) == 0)
	{
		OutValue = EOnlineServices::Nintendo;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_5")) == 0)
	{
		OutValue = EOnlineServices::Reserved_5;
	}
	else if (FCString::Stricmp(InStr, TEXT("Steam")) == 0)
	{
		OutValue = EOnlineServices::Steam;
	}
	else if (FCString::Stricmp(InStr, TEXT("Google")) == 0)
	{
		OutValue = EOnlineServices::Google;
	}
	else if (FCString::Stricmp(InStr, TEXT("GooglePlay")) == 0)
	{
		OutValue = EOnlineServices::GooglePlay;
	}
	else if (FCString::Stricmp(InStr, TEXT("Apple")) == 0)
	{
		OutValue = EOnlineServices::Apple;
	}
	else if (FCString::Stricmp(InStr, TEXT("AppleGameKit")) == 0)
	{
		OutValue = EOnlineServices::AppleGameKit;
	}
	else if (FCString::Stricmp(InStr, TEXT("Samsung")) == 0)
	{
		OutValue = EOnlineServices::Samsung;
	}
	else if (FCString::Stricmp(InStr, TEXT("Oculus")) == 0)
	{
		OutValue = EOnlineServices::Oculus;
	}
	else if (FCString::Stricmp(InStr, TEXT("Tencent")) == 0)
	{
		OutValue = EOnlineServices::Tencent;
	}
	else if (FCString::Stricmp(InStr, TEXT("EpicAccount")) == 0)
	{
		OutValue = EOnlineServices::EpicAccount;
	}
	else if (FCString::Stricmp(InStr, TEXT("EpicGame")) == 0)
	{
		OutValue = EOnlineServices::EpicGame;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_16")) == 0)
	{
		OutValue = EOnlineServices::Reserved_16;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_17")) == 0)
	{
		OutValue = EOnlineServices::Reserved_17;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_18")) == 0)
	{
		OutValue = EOnlineServices::Reserved_18;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_19")) == 0)
	{
		OutValue = EOnlineServices::Reserved_19;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_20")) == 0)
	{
		OutValue = EOnlineServices::Reserved_20;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_21")) == 0)
	{
		OutValue = EOnlineServices::Reserved_21;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_22")) == 0)
	{
		OutValue = EOnlineServices::Reserved_22;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_23")) == 0)
	{
		OutValue = EOnlineServices::Reserved_23;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_24")) == 0)
	{
		OutValue = EOnlineServices::Reserved_24;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_25")) == 0)
	{
		OutValue = EOnlineServices::Reserved_25;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_26")) == 0)
	{
		OutValue = EOnlineServices::Reserved_26;
	}
	else if (FCString::Stricmp(InStr, TEXT("Reserved_27")) == 0)
	{
		OutValue = EOnlineServices::Reserved_27;
	}
	else if (FCString::Stricmp(InStr, TEXT("GameDefined_0")) == 0)
	{
		OutValue = EOnlineServices::GameDefined_0;
	}
	else if (FCString::Stricmp(InStr, TEXT("GameDefined_1")) == 0)
	{
		OutValue = EOnlineServices::GameDefined_1;
	}
	else if (FCString::Stricmp(InStr, TEXT("GameDefined_2")) == 0)
	{
		OutValue = EOnlineServices::GameDefined_2;
	}
	else if (FCString::Stricmp(InStr, TEXT("GameDefined_3")) == 0)
	{
		OutValue = EOnlineServices::GameDefined_3;
	}
	else if (FCString::Stricmp(InStr, TEXT("None")) == 0)
	{
		OutValue = EOnlineServices::None;
	}
	else if (FCString::Stricmp(InStr, TEXT("Platform")) == 0)
	{
		OutValue = EOnlineServices::Platform;
	}
	else if (FCString::Stricmp(InStr, TEXT("Default")) == 0)
	{
		OutValue = EOnlineServices::Default;
	}
	else
	{
		for (uint32 Index = 0; Index < OnlineServicesAliasArraySize; ++Index)
		{
			const TCHAR* Alias = &OnlineServicesAliasArray[Index][0];
			if (FCString::Stricmp(InStr, Alias) == 0)
			{
				OutValue = EOnlineServices((uint32)EOnlineServices::GameDefined_0 + Index);
				return;
			}
		}

		// Empty string is ok and gets mapped to "None". Anything else implies we are missing a case.
		check(FCString::Strlen(InStr) == 0);
		OutValue = EOnlineServices::None;
	}
}

bool TEMP_ShouldUseEpicModularPlugins()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("EnableEOSSplitPlugins")))
	{
		return true;
	}

	bool bShouldUseEpicModularPlugins = false;
	GConfig->GetBool(TEXT("CoreOnline"), TEXT("bShouldUseEpicModularPlugins"), bShouldUseEpicModularPlugins, GEngineIni);
	return bShouldUseEpicModularPlugins;
}

FOnlineIdRegistryRegistry& FOnlineIdRegistryRegistry::Get()
{
	return TLazySingleton<FOnlineIdRegistryRegistry>::Get();
}

FOnlineIdRegistryRegistry::FOnlineIdRegistryRegistry()
	: ForeignAccountIdRegistry(MakeUnique<FOnlineForeignAccountIdRegistry>())
{
}

FOnlineIdRegistryRegistry::~FOnlineIdRegistryRegistry()
{
}

void FOnlineIdRegistryRegistry::TearDown()
{
	return TLazySingleton<FOnlineIdRegistryRegistry>::TearDown();
}

void FOnlineIdRegistryRegistry::RegisterAccountIdRegistry(EOnlineServices OnlineServices, IOnlineAccountIdRegistry* Registry, int32 Priority)
{
	FAccountIdRegistryAndPriority* Found = AccountIdRegistries.Find(OnlineServices);
	if (Found == nullptr || Found->Priority < Priority)
	{
		AccountIdRegistries.Emplace(OnlineServices, FAccountIdRegistryAndPriority(Registry, Priority));
	}
}

void FOnlineIdRegistryRegistry::UnregisterAccountIdRegistry(EOnlineServices OnlineServices, int32 Priority)
{
	FAccountIdRegistryAndPriority* Found = AccountIdRegistries.Find(OnlineServices);
	if (Found != nullptr && Found->Priority == Priority)
	{
		AccountIdRegistries.Remove(OnlineServices);
	}
}

FString FOnlineIdRegistryRegistry::ToString(const FAccountId& AccountId) const
{
	FString Result;
	if (IOnlineAccountIdRegistry* Registry = GetAccountIdRegistry(AccountId.GetOnlineServicesType()))
	{
		Result = Registry->ToString(AccountId);
	}
	else
	{
		Result = ForeignAccountIdRegistry->ToString(AccountId);
	}
	return Result;
}

FString FOnlineIdRegistryRegistry::ToLogString(const FAccountId& AccountId) const
{
	FString Result;
	if (IOnlineAccountIdRegistry* Registry = GetAccountIdRegistry(AccountId.GetOnlineServicesType()))
	{
		Result = Registry->ToLogString(AccountId);
	}
	else
	{
		Result = ForeignAccountIdRegistry->ToLogString(AccountId);
	}
	return Result;
}

TArray<uint8> FOnlineIdRegistryRegistry::ToReplicationData(const FAccountId& AccountId) const
{
	TArray<uint8> RepData;
	if (IOnlineAccountIdRegistry* Registry = GetAccountIdRegistry(AccountId.GetOnlineServicesType()))
	{
		RepData = Registry->ToReplicationData(AccountId);
	}
	else
	{
		RepData = ForeignAccountIdRegistry->ToReplicationData(AccountId);
	}
	return RepData;
}

FAccountId FOnlineIdRegistryRegistry::ToAccountId(EOnlineServices Services, const TArray<uint8>& RepData) const
{
	FAccountId AccountId;
	if (IOnlineAccountIdRegistry* Registry = GetAccountIdRegistry(Services))
	{
		AccountId = Registry->FromReplicationData(RepData);
	}
	else
	{
		AccountId = ForeignAccountIdRegistry->FromReplicationData(Services, RepData);
	}
	return AccountId;
}

FAccountId FOnlineIdRegistryRegistry::ToAccountId(EOnlineServices Services, const FString& StringData) const
{
	FAccountId AccountId;
	if (IOnlineAccountIdRegistry* Registry = GetAccountIdRegistry(Services))
	{
		AccountId = Registry->FromStringData(StringData);
	}

	return AccountId;
}

IOnlineAccountIdRegistry* FOnlineIdRegistryRegistry::GetAccountIdRegistry(EOnlineServices OnlineServices) const
{
	IOnlineAccountIdRegistry* Registry = nullptr;
	if (const FAccountIdRegistryAndPriority* Found = AccountIdRegistries.Find(OnlineServices))
	{
		Registry = Found->Registry;
	}
	return Registry;
}

void FOnlineIdRegistryRegistry::RegisterSessionIdRegistry(EOnlineServices OnlineServices, IOnlineSessionIdRegistry* Registry, int32 Priority)
{
	FSessionIdRegistryAndPriority* Found = SessionIdRegistries.Find(OnlineServices);
	if (Found == nullptr || Found->Priority < Priority)
	{
		SessionIdRegistries.Emplace(OnlineServices, FSessionIdRegistryAndPriority(Registry, Priority));
	}
}

void FOnlineIdRegistryRegistry::UnregisterSessionIdRegistry(EOnlineServices OnlineServices, int32 Priority)
{
	FSessionIdRegistryAndPriority* Found = SessionIdRegistries.Find(OnlineServices);
	if (Found != nullptr && Found->Priority == Priority)
	{
		SessionIdRegistries.Remove(OnlineServices);
	}
}

FString FOnlineIdRegistryRegistry::ToString(const FOnlineSessionId& SessionId) const
{
	FString Result;
	if (IOnlineSessionIdRegistry* Registry = GetSessionIdRegistry(SessionId.GetOnlineServicesType()))
	{
		Result = Registry->ToString(SessionId);
	}
	return Result;
}

FString FOnlineIdRegistryRegistry::ToLogString(const FOnlineSessionId& SessionId) const
{
	FString Result;
	if (IOnlineSessionIdRegistry* Registry = GetSessionIdRegistry(SessionId.GetOnlineServicesType()))
	{
		Result = Registry->ToLogString(SessionId);
	}
	return Result;
}

TArray<uint8> FOnlineIdRegistryRegistry::ToReplicationData(const FOnlineSessionId& SessionId) const
{
	TArray<uint8> RepData;
	if (IOnlineSessionIdRegistry* Registry = GetSessionIdRegistry(SessionId.GetOnlineServicesType()))
	{
		RepData = Registry->ToReplicationData(SessionId);
	}
	return RepData;
}

FOnlineSessionId FOnlineIdRegistryRegistry::ToSessionId(EOnlineServices Services, const TArray<uint8>& RepData) const
{
	FOnlineSessionId SessionId;
	if (IOnlineSessionIdRegistry* Registry = GetSessionIdRegistry(Services))
	{
		SessionId = Registry->FromReplicationData(RepData);
	}
	return SessionId;
}

IOnlineSessionIdRegistry* FOnlineIdRegistryRegistry::GetSessionIdRegistry(EOnlineServices OnlineServices) const
{
	IOnlineSessionIdRegistry* Registry = nullptr;
	if (const FSessionIdRegistryAndPriority* Found = SessionIdRegistries.Find(OnlineServices))
	{
		Registry = Found->Registry;
	}
	return Registry;
}

void FOnlineIdRegistryRegistry::RegisterSessionInviteIdRegistry(EOnlineServices OnlineServices, IOnlineSessionInviteIdRegistry* Registry, int32 Priority)
{
	FSessionInviteIdRegistryAndPriority* Found = SessionInviteIdRegistries.Find(OnlineServices);
	if (Found == nullptr || Found->Priority < Priority)
	{
		SessionInviteIdRegistries.Emplace(OnlineServices, FSessionInviteIdRegistryAndPriority(Registry, Priority));
	}
}

void FOnlineIdRegistryRegistry::UnregisterSessionInviteIdRegistry(EOnlineServices OnlineServices, int32 Priority)
{
	FSessionInviteIdRegistryAndPriority* Found = SessionInviteIdRegistries.Find(OnlineServices);
	if (Found != nullptr && Found->Priority == Priority)
	{
		SessionInviteIdRegistries.Remove(OnlineServices);
	}
}

FString FOnlineIdRegistryRegistry::ToLogString(const FSessionInviteId& SessionInviteId) const
{
	FString Result;
	if (IOnlineSessionInviteIdRegistry* Registry = GetSessionInviteIdRegistry(SessionInviteId.GetOnlineServicesType()))
	{
		Result = Registry->ToLogString(SessionInviteId);
	}
	return Result;
}

TArray<uint8> FOnlineIdRegistryRegistry::ToReplicationData(const FSessionInviteId& SessionInviteId) const
{
	TArray<uint8> RepData;
	if (IOnlineSessionInviteIdRegistry* Registry = GetSessionInviteIdRegistry(SessionInviteId.GetOnlineServicesType()))
	{
		RepData = Registry->ToReplicationData(SessionInviteId);
	}
	return RepData;
}

FSessionInviteId FOnlineIdRegistryRegistry::ToSessionInviteId(EOnlineServices Services, const TArray<uint8>& RepData) const
{
	FSessionInviteId SessionInviteId;
	if (IOnlineSessionInviteIdRegistry* Registry = GetSessionInviteIdRegistry(Services))
	{
		SessionInviteId = Registry->FromReplicationData(RepData);
	}
	return SessionInviteId;
}

IOnlineSessionInviteIdRegistry* FOnlineIdRegistryRegistry::GetSessionInviteIdRegistry(EOnlineServices OnlineServices) const
{
	IOnlineSessionInviteIdRegistry* Registry = nullptr;
	if (const FSessionInviteIdRegistryAndPriority* Found = SessionInviteIdRegistries.Find(OnlineServices))
	{
		Registry = Found->Registry;
	}
	return Registry;
}

template<typename IdType>
FString ToStringImpl(const IdType& Id)
{
	return FOnlineIdRegistryRegistry::Get().ToString(Id);
}

FString ToString(const FAccountId& Id) { return ToStringImpl(Id); }
FString ToString(const FOnlineSessionId& Id) { return ToStringImpl(Id); }
// TODO
//FString ToString(const FLobbyId& Id) { return ToStringImpl(Id); }
//FString ToString(const FSessionInviteId& Id) { return ToStringImpl(Id); }
//FString ToString(const FVerifiedAuthTicketId& Id) { return ToStringImpl(Id); }
//FString ToString(const FVerifiedAuthSessionId& Id) { return ToStringImpl(Id); }

template<typename IdType>
FString ToLogStringImpl(const IdType& Id)
{
	if constexpr (std::is_same_v<IdType, FAccountId>
		|| std::is_same_v<IdType, FOnlineSessionId>
		|| std::is_same_v<IdType, FSessionInviteId>)
		// TODO Add FLobbyId
	{
		return FString::Printf(TEXT("%s:%d (%s)"), LexToString(Id.GetOnlineServicesType()), Id.GetHandle(), *FOnlineIdRegistryRegistry::Get().ToLogString(Id));
	}
	else
	{
		return FString::Printf(TEXT("%s:%d"), LexToString(Id.GetOnlineServicesType()), Id.GetHandle());
	}
}

FString ToLogString(const FAccountId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FLobbyId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FOnlineSessionId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FSessionInviteId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FVerifiedAuthTicketId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FVerifiedAuthSessionId& Id) { return ToLogStringImpl(Id); }
FString ToLogString(const FLoginContinuationId& Id) { return ToLogStringImpl(Id); }

}	/* UE::Online */

FString FUniqueNetIdWrapper::ToString() const
{
	FString Result;
	if (IsValid())
	{
		if (IsV1())
		{
			Result = GetV1Unsafe()->ToString();
		}
		else if (IsV2())
		{
			Result = UE::Online::ToString(GetV2Unsafe());
		}
	}
	return Result;
}

FString FUniqueNetIdWrapper::ToDebugString() const
{
	FString Result;
	if (IsValid())
	{
		if (IsV1())
		{
			const FUniqueNetIdPtr& Ptr = GetV1();
			Result = FString::Printf(TEXT("%s:%s"), *Ptr->GetType().ToString(), *Ptr->ToDebugString());
		}
		else
		{
			Result = ToLogString(GetV2());
		}
	}
	else
	{
		Result = TEXT("INVALID");
	}
	return Result;
}
