// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/CoreOnlinePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CoreOnlinePrivate)

namespace UE::Online {

FString FOnlineForeignAccountIdRegistry::ToString(const FAccountId& AccountId) const
{
	FString Result;
	if (AccountId.IsValid())
	{
		const EOnlineServices HandleType = AccountId.GetOnlineServicesType();
		const uint32 HandleIndex = AccountId.GetHandle() - 1;
		if (const FRepData* RepDataForServices = OnlineServicesToRepData.Find(HandleType))
		{
			const TArray<TArray<uint8>>& RepDataArray = RepDataForServices->RepDataArray;
			if (ensure(RepDataArray.IsValidIndex(HandleIndex)))
			{
				const TArray<uint8>& RepData = RepDataArray[HandleIndex];
				// TODO really this requires traits to interpret the bytes as a string/int/whatever
				Result = FString::FromHexBlob(RepData.GetData(), RepData.Num());
			}
		}
	}
	return Result;
}

FString FOnlineForeignAccountIdRegistry::ToLogString(const FAccountId& AccountId) const
{
	FString Result;
	if (AccountId.IsValid())
	{
		const EOnlineServices HandleType = AccountId.GetOnlineServicesType();
		const uint32 HandleIndex = AccountId.GetHandle() - 1;
		if (const FRepData* RepDataForServices = OnlineServicesToRepData.Find(HandleType))
		{
			const TArray<TArray<uint8>>& RepDataArray = RepDataForServices->RepDataArray;
			if (ensure(RepDataArray.IsValidIndex(HandleIndex)))
			{
				const TArray<uint8>& RepData = RepDataArray[HandleIndex];
				Result = FString::Printf(TEXT("ForeignId=[Type=%d Handle=%d RepData=[%s]"),
					EnumToUnderlyingType(HandleType),
					HandleIndex,
					*FString::FromHexBlob(RepData.GetData(), RepData.Num()));
			}
		}
	}
	return Result;
}

TArray<uint8> FOnlineForeignAccountIdRegistry::ToReplicationData(const FAccountId& AccountId) const
{
	TArray<uint8> RepData;
	if (AccountId.IsValid())
	{
		const EOnlineServices HandleType = AccountId.GetOnlineServicesType();
		const uint32 HandleIndex = AccountId.GetHandle() - 1;
		if (const FRepData* RepDataForServices = OnlineServicesToRepData.Find(AccountId.GetOnlineServicesType()))
		{
			const TArray<TArray<uint8>>& RepDataArray = RepDataForServices->RepDataArray;
			if (ensure(RepDataArray.IsValidIndex(HandleIndex)))
			{
				RepData = RepDataArray[HandleIndex];
			}
		}
	}
	return RepData;
}

FAccountId FOnlineForeignAccountIdRegistry::FromReplicationData(EOnlineServices Services, const TArray<uint8>& RepData)
{
	FAccountId AccountId;
	if (RepData.Num())
	{
		FRepData& RepDataForServices = OnlineServicesToRepData.FindOrAdd(Services);
		if (FAccountId* Found = RepDataForServices.RepDataToHandle.Find(RepData))
		{
			AccountId = *Found;
		}
		else
		{
			RepDataForServices.RepDataArray.Add(RepData);
			AccountId = FAccountId(Services, RepDataForServices.RepDataArray.Num());
			RepDataForServices.RepDataToHandle.Emplace(RepData, AccountId);
		}
	}
	return AccountId;
}

}
