// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "UObject/ObjectMacros.h"

#include "CoreOnlinePrivate.generated.h"

// This exists purely to guarantee a package is created, or the engine will not boot.
UENUM(BlueprintType)
enum class ECoreOnlineDummy : uint8
{
	Dummy
};

static uint32 GetTypeHash(const TArray<uint8>& Array)
{
	uint32 Seed = 0;
	for (const uint8& Elem : Array)
		Seed ^= GetTypeHash(Elem) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
	return Seed;
}

namespace UE::Online {

/**
 * Registry for storing account id's for unregistered services implementations.
 */
class FOnlineForeignAccountIdRegistry
{
public:
	FString ToString(const FAccountId& AccountId) const;
	FString ToLogString(const FAccountId& AccountId) const;
	TArray<uint8> ToReplicationData(const FAccountId& AccountId) const;
	FAccountId FromReplicationData(EOnlineServices Services, const TArray<uint8>& RepData);

private:
	struct FRepData
	{
		TMap<TArray<uint8>, FAccountId> RepDataToHandle;
		TArray<TArray<uint8>> RepDataArray;
	};
	TMap<EOnlineServices, FRepData> OnlineServicesToRepData;
};

} // namespace UE::Online