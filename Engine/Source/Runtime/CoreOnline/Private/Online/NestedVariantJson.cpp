// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/NestedVariantJson.h"

#include "Dom/JsonObject.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"

using FJsonWriterFactory = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>;
using FJsonWriter = TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>;
using FJsonSerializerNestedVariant = TJsonSerializer<FJsonSerializerPolicy_NestedVariant>;

FString NestedVariantToJson(const FNestedVariantJson::FMapPtr& Map)
{
	FString Result;

	if (!Map)
	{
		return Result;
	}

	TSharedRef<FJsonWriter> Writer = FJsonWriterFactory::Create(&Result);
	verify(FJsonSerializerNestedVariant::Serialize(Map, Writer));

	return Result;
}

TSharedRef<FJsonObject> NestedVariantToJsonObject(const FNestedVariantJson::FMapRef& Map)
{
	const FString& JsonStr = NestedVariantToJson(Map);
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonStr);	
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	verify(FJsonSerializer::Deserialize(Reader, JsonObject));

	return JsonObject.ToSharedRef();
}

TSharedRef<FJsonObject> NestedVariantToJsonObject(const FNestedVariantJson::FMap& Map)
{
	const FNestedVariantJson::FMapRef Ref = MakeShared<FNestedVariantJson::FMap>(Map);
	return NestedVariantToJsonObject(Ref);
}

void NestedVariantFromJson(const char* InJson, FNestedVariantJson::FMapRef& Map)
{
	if (!InJson || strlen(InJson) == 0)
	{
		return;
	}

	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(UTF8_TO_TCHAR(InJson));
	FNestedVariantJson::FMapPtr MapPtr;
	verify(FJsonSerializerNestedVariant::Deserialize(Reader, MapPtr));

	if (MapPtr.IsValid())
	{
		Map->Append(MoveTemp(*MapPtr));
	}
}

void NestedVariantFromJson(const char* InJson, FNestedVariantJson::FMap& Map)
{
	FNestedVariantJson::FMapRef Ref = MakeShared<FNestedVariantJson::FMap>();
	NestedVariantFromJson(InJson, Ref);
	Map = MoveTemp(*Ref);
}

void NestedVariantFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, FNestedVariantJson::FMapRef& Map)
{
	FString JsonStr;
	TSharedRef<FJsonWriter> Writer = FJsonWriterFactory::Create(&JsonStr);
	verify(FJsonSerializer::Serialize(JsonObject, Writer));

	NestedVariantFromJson(TCHAR_TO_UTF8(*JsonStr), Map);
}

void NestedVariantFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, FNestedVariantJson::FMap& Map)
{
	FNestedVariantJson::FMapRef Ref = MakeShared<FNestedVariantJson::FMap>(Map);
	return NestedVariantFromJsonObject(JsonObject, Ref);
}

void LexFromString(FNestedVariantJson::FMap& OutValue, const TCHAR* InString)
{
	FNestedVariantJson::FMapRef  OutValueRef = FNestedVariantJson::FMap::CreateVariant();
	NestedVariantFromJson(TCHAR_TO_UTF8(InString), OutValueRef);

	OutValue = OutValueRef.Get();
}

bool FJsonSerializerPolicy_NestedVariant::GetValueFromState(const FStackState& State, FValue& OutValue)
{
	switch (State.Type)
	{
	case EJson::Object:
	{
		if (State.Object.IsValid())
		{
			OutValue.Set<FNestedVariantJson::FMapRef>(State.Object.ToSharedRef());
			return true;
		}
	}		
	case EJson::Array:
	{
		if (State.Array.IsValid())
		{
			OutValue.Set<FNestedVariantJson::FArrayRef>(State.Array.ToSharedRef());
			return true;
		}
	}
	case EJson::None:
	case EJson::Null:
	case EJson::String:
	case EJson::Number:
	case EJson::Boolean:
	default:
		// FIXME: would be nice to handle non-composite root values but StackState Deserialize just drops them on the floor
		break;
	}

	return false;
}

bool FJsonSerializerPolicy_NestedVariant::GetValueFromState(const FStackState& State, FArrayOfValues& OutArray)
{
	// Returning an empty array is ok
	if (State.Type != EJson::Array)
	{
		return false;
	}

	OutArray = State.Array;

	return true;
}

bool FJsonSerializerPolicy_NestedVariant::GetValueFromState(const FStackState& State, FMapOfValues& OutMap)
{
	// Returning an empty object is ok
	if (State.Type != EJson::Object)
	{
		return false;
	}

	OutMap = State.Object;

	return true;
}

void FJsonSerializerPolicy_NestedVariant::ResetValue(FValue& OutValue)
{
	OutValue = FValue();
}

void FJsonSerializerPolicy_NestedVariant::ReadObjectStart(FStackState& State)
{
	State.Type = EJson::Object;
	State.Object = FNestedVariantJson::FMap::CreateVariant();
}

void FJsonSerializerPolicy_NestedVariant::ReadObjectEnd(FStackState& State, FValue& OutValue)
{
	OutValue.Set<FNestedVariantJson::FMapRef>(State.Object.ToSharedRef());
}

void FJsonSerializerPolicy_NestedVariant::ReadArrayStart(FStackState& State)
{
	State.Type = EJson::Array;
	State.Array = FNestedVariantJson::FArray::CreateVariant();
}

void FJsonSerializerPolicy_NestedVariant::ReadArrayEnd(FStackState& State, FValue& OutValue)
{
	OutValue.Set<FNestedVariantJson::FArrayRef>(State.Array.ToSharedRef());
}

void FJsonSerializerPolicy_NestedVariant::ReadNull(FValue& OutValue)
{
}

void FJsonSerializerPolicy_NestedVariant::AddValueToObject(FStackState& State, const FString& Identifier, FValue& NewValue)
{
	State.Object->Add(Identifier, NewValue);
}

void FJsonSerializerPolicy_NestedVariant::AddValueToArray(FStackState& State, FValue& NewValue)
{
	State.Array->Add(NewValue);
}