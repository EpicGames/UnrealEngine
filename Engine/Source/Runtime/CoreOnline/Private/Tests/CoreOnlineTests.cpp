// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Online/CoreOnline.h"
#include "Online/CoreOnlinePrivate.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FForeignIdRepTest,
	"System.Engine.Online.ForeignIdRepTest",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
	bool FForeignIdRepTest::RunTest(const FString& Parameters)
{
	using namespace UE::Online;
	const EOnlineServices Services = EOnlineServices::Apple;
	const TArray<uint8> SourceRepData { 0, 1, 2, 3, 4, 5, 6, 7 };

	FOnlineIdRegistryRegistry Registry;
	const FAccountId AccountId = Registry.ToAccountId(Services, SourceRepData);
	const TArray<uint8> RepData = Registry.ToReplicationData(AccountId);
	UTEST_EQUAL(TEXT(""), RepData, SourceRepData);

	return true;
}