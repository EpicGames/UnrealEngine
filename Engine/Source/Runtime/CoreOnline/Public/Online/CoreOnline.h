// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Hash/CityHash.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TVariant.h"
#include "Online/CoreOnlineFwd.h"
#include "Online/CoreOnlinePackage.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#include "CoreOnline.generated.h"

class FDefaultSetAllocator;
class FLazySingleton;

/** Maximum players supported on a given platform */
#if !defined(MAX_LOCAL_PLAYERS)
	#if PLATFORM_MAX_LOCAL_PLAYERS
		#define MAX_LOCAL_PLAYERS PLATFORM_MAX_LOCAL_PLAYERS
	#elif PLATFORM_DESKTOP
		#define MAX_LOCAL_PLAYERS 4
	#else
		#define MAX_LOCAL_PLAYERS 1
	#endif
#endif //MAX_LOCAL_PLAYERS


USTRUCT(noexport)
struct FJoinabilitySettings
{
	//GENERATED_BODY()

	/** Name of session these settings affect */
	UPROPERTY()
	FName SessionName;
	/** Is this session now publicly searchable */
	UPROPERTY()
	bool bPublicSearchable;
	/** Does this session allow invites */
	UPROPERTY()
	bool bAllowInvites;
	/** Does this session allow public join via presence */
	UPROPERTY()
	bool bJoinViaPresence;
	/** Does this session allow friends to join via presence */
	UPROPERTY()
	bool bJoinViaPresenceFriendsOnly;
	/** Current max players in this session */
	UPROPERTY()
	int32 MaxPlayers;
	/** Current max party size in this session */
	UPROPERTY()
	int32 MaxPartySize;

	FJoinabilitySettings() :
		SessionName(NAME_None),
		bPublicSearchable(false),
		bAllowInvites(false),
		bJoinViaPresence(false),
		bJoinViaPresenceFriendsOnly(false),
		MaxPlayers(0),
		MaxPartySize(0)
	{
	}

	bool operator==(const FJoinabilitySettings& Other) const
	{
		return SessionName == Other.SessionName &&
			bPublicSearchable == Other.bPublicSearchable &&
			bAllowInvites == Other.bAllowInvites &&
			bJoinViaPresence == Other.bJoinViaPresence &&
			bJoinViaPresenceFriendsOnly == Other.bJoinViaPresenceFriendsOnly &&
			MaxPlayers == Other.MaxPlayers &&
			MaxPartySize == Other.MaxPartySize;
	}

	bool operator!=(const FJoinabilitySettings& Other) const
	{
		return !(FJoinabilitySettings::operator==(Other));
	}
};

/**
 * Abstraction of a profile service online Id
 * The class is meant to be opaque
 */
class FUniqueNetId : public TSharedFromThis<FUniqueNetId>
{
protected:

	/** Only constructible by derived type */
	FUniqueNetId() = default;

	FUniqueNetId(const FUniqueNetId& Src) = default;
	FUniqueNetId& operator=(const FUniqueNetId& Src) = default;

	virtual bool Compare(const FUniqueNetId& Other) const
	{
		return (GetType() == Other.GetType() &&
			GetSize() == Other.GetSize() &&
			FMemory::Memcmp(GetBytes(), Other.GetBytes(), GetSize()) == 0);
	}

public:

	virtual ~FUniqueNetId() = default;

	/**
	 *	Comparison operator
	 */
	friend bool operator==(const FUniqueNetId& Lhs, const FUniqueNetId& Rhs)
	{
		return Lhs.Compare(Rhs);
	}

	friend bool operator!=(const FUniqueNetId& Lhs, const FUniqueNetId& Rhs)
	{
		return !Lhs.Compare(Rhs);
	}

	/**
	 * Get the type token for this opaque data
	 * This is useful for inferring UniqueId subclasses and knowing which OSS it "goes with"
	 *
	 * @return FName representing the Type
	 */
	virtual FName GetType() const { return NAME_None; /* This should be pure virtual, however, older versions of the OSS plugins cannot handle that */ }

	/** 
	 * Get the raw byte representation of this opaque data
	 * This data is platform dependent and shouldn't be manipulated directly
	 *
	 * @return byte array of size GetSize()
	 */
	virtual const uint8* GetBytes() const = 0;

	/** 
	 * Get the size of the opaque data
	 *
	 * @return size in bytes of the data representation
	 */
	virtual int32 GetSize() const = 0;

	/**
	 * Check the validity of the opaque data
	 *
	 * @return true if this is well formed data, false otherwise
	 */
	virtual bool IsValid() const = 0;

	/**
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form
	 */
	virtual FString ToString() const = 0;

	/**
	 * Get a human readable representation of the opaque data
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return data in string form
	 */
	virtual FString ToDebugString() const = 0;

	virtual uint32 GetTypeHash() const
	{
		return CityHash32(reinterpret_cast<const char*>(GetBytes()), GetSize());
	}

	/**
	 * @return hex encoded string representation of unique id
	 */
	FString GetHexEncodedString() const
	{
		if (GetSize() > 0 && GetBytes() != NULL)
		{
			return BytesToHex(GetBytes(), GetSize());
		}
		return FString();
	}

	friend inline uint32 GetTypeHash(const FUniqueNetId& Value)
	{
		return Value.GetTypeHash();
	}
};

namespace UE::Online {

class FOnlineForeignAccountIdRegistry;

/** Tags used as template argument to TOnlineId to make it a compile error to assign between id's of different types */
namespace OnlineIdHandleTags
{
	struct FAccount {};
	struct FSession {};
	struct FSessionInvite {};
	struct FLobby {};
	struct FVerifiedAuthTicket {};
	struct FVerifiedAuthSession {};
	struct FLoginContinuation {};
}

enum class EOnlineServices : uint8
{
	// Null, Providing minimal functionality when no backend services are required
	Null,
	// Epic Online Services, merges Epic Account Services and Epic Game Services. Note this will be deprecated in favor of separating out the services
	Epic,
	// Xbox services
	Xbox,
	// PlayStation Network
	PSN,
	// Nintendo
	Nintendo,
	// Unused,
	Reserved_5,
	// Steam
	Steam,
	// Google
	Google,
	// GooglePlay
	GooglePlay,
	// Apple
	Apple,
	// GameKit
	AppleGameKit,
	// Samsung
	Samsung,
	// Oculus
	Oculus,
	// Tencent
	Tencent,
	// Epic Account Services (friends, presence, etc)
	EpicAccount,
	// Epic Game Services (lobbies, achievements, etc)
	EpicGame,
	// Reserved for future use/platform extensions
	Reserved_16,
	Reserved_17,
	Reserved_18,
	Reserved_19,
	Reserved_20,
	Reserved_21,
	Reserved_22,
	Reserved_23,
	Reserved_24,
	Reserved_25,
	Reserved_26,
	Reserved_27,
	// For game specific Online Services
	// To make use of GameDefined_x, ideally it should do all these steps in module startup:
	//	1. Read the EOnlineServices value from config, to avoid conflict with any other plugin while keeping it data-driven;
	//			EOnlineServices XxxServices = EOnlineServices::GameDefined_0;
	//			FString ConfigStr;
	//			if (GConfig->GetString(TEXT("OnlineServicesXxx"), TEXT("ServicesEnumEntry"), ConfigStr, GEngineIni))
	//			{
	//				LexFromString(XxxServices, *ConfigStr);
	//			}
	//	2. Register the factory with that value;
	//			FOnlineServicesRegistry::Get().RegisterServicesFactory(XxxServices, ...);
	//	3. Optionally register the alias based on the value read from config  
	//			RegisterAliasForGameDefined(XxxServices, TEXT("Xxx"));
	GameDefined_0 = 28,
	GameDefined_1,
	GameDefined_2,
	GameDefined_3,
	// None, used internally to resolve Platform or Default if they are not configured
	None = 253,
	// Platform native, may not exist for all platforms
	Platform = 254,
	// Default, configured via ini, TODO: List specific ini section/key
	Default = 255
};

COREONLINE_API void RegisterAliasForGameDefined(EOnlineServices Value, const TCHAR* InStr);
COREONLINE_API const TCHAR* LexToString(EOnlineServices Value);
COREONLINE_API void LexFromString(EOnlineServices& OutValue, const TCHAR* InStr);

/**
 * Temporary
 * Signals that the EOnlineServices::Epic monolithic plugins should not be used, and EOnlineServices::EpicAccount/EpicGame should be used instead.
 * This is a stepping stone to promoting the EOnlineServices::EpicAccount/EpicGame modular plugins and deprecating the monolithic ones.
 */
COREONLINE_API bool TEMP_ShouldUseEpicModularPlugins();

/**
 * A handle to an id which uniquely identifies a persistent or transient online resource, i.e. account/session/party etc, within a given Online Services provider.
 * At most one id, and therefore one handle, exists for any given resource. The id and handle persist until the OnlineServices module is unloaded.
 * Passed to and returned from OnlineServices APIs.
 */
template<typename IdType>
class TOnlineId
{
public:
	TOnlineId() = default;
	TOnlineId(EOnlineServices Type, uint32 Handle)
	{
		check(Handle < 0xFF000000);
		Value = (Handle & 0x00FFFFFF) | (uint32(Type) << 24);
	}

	inline bool IsValid() const { return GetHandle() != 0; }
	inline explicit operator bool() const { return IsValid(); }

	EOnlineServices GetOnlineServicesType() const { return EOnlineServices(Value >> 24); }
	uint32 GetHandle() const { return Value & 0x00FFFFFF; }

	bool operator==(const TOnlineId& Other) const { return Value == Other.Value; }
	bool operator!=(const TOnlineId& Other) const { return Value != Other.Value; }

private:
	uint32 Value = uint32(EOnlineServices::Null) << 24;
};

using FAccountId = TOnlineId<OnlineIdHandleTags::FAccount>;
using FLobbyId = TOnlineId<OnlineIdHandleTags::FLobby>;
// TODO rename to FSessionId
using FOnlineSessionId = TOnlineId<OnlineIdHandleTags::FSession>;
using FSessionInviteId = TOnlineId<OnlineIdHandleTags::FSessionInvite>;
using FVerifiedAuthTicketId = TOnlineId<OnlineIdHandleTags::FVerifiedAuthTicket>;
using FVerifiedAuthSessionId = TOnlineId<OnlineIdHandleTags::FVerifiedAuthSession>;
using FLoginContinuationId = TOnlineId<OnlineIdHandleTags::FLoginContinuation>;

COREONLINE_API FString ToString(const FAccountId& Id);
COREONLINE_API FString ToString(const FOnlineSessionId& Id);
// TODO
// COREONLINE_API FString ToString(const FLobbyId& Id);
// COREONLINE_API FString ToString(const FSessionInviteId& Id);
// COREONLINE_API FString ToString(const FVerifiedAuthTicketId& Id);
// COREONLINE_API FString ToString(const FVerifiedAuthSessionId& Id);

COREONLINE_API FString ToLogString(const FAccountId& Id);
COREONLINE_API FString ToLogString(const FLobbyId& Id);
COREONLINE_API FString ToLogString(const FOnlineSessionId& Id);
COREONLINE_API FString ToLogString(const FSessionInviteId& Id);
COREONLINE_API FString ToLogString(const FVerifiedAuthTicketId& Id);
COREONLINE_API FString ToLogString(const FVerifiedAuthSessionId& Id);
COREONLINE_API FString ToLogString(const FLoginContinuationId& Id);

template<typename IdType>
inline uint32 GetTypeHash(const TOnlineId<IdType>& OnlineId)
{
	using ::GetTypeHash;
	return HashCombine(GetTypeHash(OnlineId.GetOnlineServicesType()), GetTypeHash(OnlineId.GetHandle()));
}

template<typename IdType>
class IOnlineIdRegistry
{
public:
	virtual ~IOnlineIdRegistry() = default;

	virtual FString ToString(const TOnlineId<IdType>& OnlineId) const = 0;
	virtual FString ToLogString(const TOnlineId<IdType>& OnlineId) const = 0;
	virtual TArray<uint8> ToReplicationData(const TOnlineId<IdType>& OnlineId) const = 0;
	virtual TOnlineId<IdType> FromReplicationData(const TArray<uint8>& ReplicationData) = 0;
	virtual TOnlineId<IdType> FromStringData(const FString& StringData) = 0;
};

using IOnlineAccountIdRegistry = IOnlineIdRegistry<OnlineIdHandleTags::FAccount>;
using IOnlineSessionIdRegistry = IOnlineIdRegistry<OnlineIdHandleTags::FSession>;
using IOnlineSessionInviteIdRegistry = IOnlineIdRegistry<OnlineIdHandleTags::FSessionInvite>;

class FOnlineIdRegistryRegistry
{
public:
	/**
	 * Get the FOnlineIdRegistryRegistry singleton
	 *
	 * @return The FOnlineIdRegistryRegistry singleton instance
	 */
	COREONLINE_API static FOnlineIdRegistryRegistry& Get();

	/**
	 * Tear down the singleton instance
	 */
	COREONLINE_API static void TearDown();

	/**
	 * Register a registry for a given OnlineServices implementation and IOnlineAccountIdHandle type
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Registry the registry of online account ids
	 * @param Priority Integer priority, allows an existing registry to be extended and registered with a higher priority so it is used instead
	 */
	COREONLINE_API void RegisterAccountIdRegistry(EOnlineServices OnlineServices, IOnlineAccountIdRegistry* Registry, int32 Priority = 0);

	/**
	 * Unregister a previously registered Account Id registry
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Priority Integer priority, will be unregistered only if the priority matches the one that is registered
	 */
	COREONLINE_API void UnregisterAccountIdRegistry(EOnlineServices OnlineServices, int32 Priority = 0);

	// TODO Might be worth these being templates
	COREONLINE_API FString ToString(const FAccountId& AccountId) const;
	COREONLINE_API FString ToLogString(const FAccountId& AccountId) const;
	COREONLINE_API TArray<uint8> ToReplicationData(const FAccountId& AccountId) const;
	COREONLINE_API FAccountId ToAccountId(EOnlineServices Services, const TArray<uint8>& RepData) const;
	COREONLINE_API FAccountId ToAccountId(EOnlineServices Services, const FString& StringData) const;

	COREONLINE_API IOnlineAccountIdRegistry* GetAccountIdRegistry(EOnlineServices OnlineServices) const;

	/**
	 * Register a registry for a given OnlineServices implementation and IOnlineSessionIdHandle type
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Registry the registry of online session ids
	 * @param Priority Integer priority, allows an existing registry to be extended and registered with a higher priority so it is used instead
	 */
	COREONLINE_API void RegisterSessionIdRegistry(EOnlineServices OnlineServices, IOnlineSessionIdRegistry* Registry, int32 Priority = 0);

	/**
	 * Unregister a previously registered Session Id registry
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Priority Integer priority, will be unregistered only if the priority matches the one that is registered
	 */
	COREONLINE_API void UnregisterSessionIdRegistry(EOnlineServices OnlineServices, int32 Priority = 0);

	COREONLINE_API FString ToString(const FOnlineSessionId& SessionId) const;
	COREONLINE_API FString ToLogString(const FOnlineSessionId& SessionId) const;
	COREONLINE_API TArray<uint8> ToReplicationData(const FOnlineSessionId& SessionId) const;
	COREONLINE_API FOnlineSessionId ToSessionId(EOnlineServices Services, const TArray<uint8>& RepData) const;

	COREONLINE_API IOnlineSessionIdRegistry* GetSessionIdRegistry(EOnlineServices OnlineServices) const;

	/**
	 * Register a registry for a given OnlineServices implementation and IOnlineSessionInviteIdHandle type
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Registry the registry of online session ids
	 * @param Priority Integer priority, allows an existing registry to be extended and registered with a higher priority so it is used instead
	 */
	COREONLINE_API void RegisterSessionInviteIdRegistry(EOnlineServices OnlineServices, IOnlineSessionInviteIdRegistry* Registry, int32 Priority = 0);

	/**
	 * Unregister a previously registered Session Invite Id registry
	 *
	 * @param OnlineServices Services that the registry is for
	 * @param Priority Integer priority, will be unregistered only if the priority matches the one that is registered
	 */
	COREONLINE_API void UnregisterSessionInviteIdRegistry(EOnlineServices OnlineServices, int32 Priority = 0);

	COREONLINE_API FString ToLogString(const FSessionInviteId& SessionInviteId) const;
	COREONLINE_API TArray<uint8> ToReplicationData(const FSessionInviteId& SessionInviteId) const;
	COREONLINE_API FSessionInviteId ToSessionInviteId(EOnlineServices Services, const TArray<uint8>& RepData) const;

	COREONLINE_API IOnlineSessionInviteIdRegistry* GetSessionInviteIdRegistry(EOnlineServices OnlineServices) const;

private:

	template<typename IdType>
	struct FOnlineIdRegistryAndPriority
	{
		FOnlineIdRegistryAndPriority(IOnlineIdRegistry<IdType>* InRegistry, int32 InPriority)
			: Registry(InRegistry), Priority(InPriority) {}

		virtual ~FOnlineIdRegistryAndPriority() = default;

		IOnlineIdRegistry<IdType>* Registry;
		int32 Priority;
	};

	typedef FOnlineIdRegistryAndPriority<OnlineIdHandleTags::FAccount> FAccountIdRegistryAndPriority;
	typedef FOnlineIdRegistryAndPriority<OnlineIdHandleTags::FSession> FSessionIdRegistryAndPriority;
	typedef FOnlineIdRegistryAndPriority<OnlineIdHandleTags::FSessionInvite> FSessionInviteIdRegistryAndPriority;

	TMap<EOnlineServices, FAccountIdRegistryAndPriority> AccountIdRegistries;
	TUniquePtr<FOnlineForeignAccountIdRegistry> ForeignAccountIdRegistry;

	TMap<EOnlineServices, FSessionIdRegistryAndPriority> SessionIdRegistries;

	TMap<EOnlineServices, FSessionInviteIdRegistryAndPriority> SessionInviteIdRegistries;

	friend FLazySingleton;

PACKAGE_SCOPE:
	FOnlineIdRegistryRegistry();
	~FOnlineIdRegistryRegistry();
};

}	/* UE::Online */

USTRUCT(noexport)
struct FUniqueNetIdWrapper
{
	//GENERATED_BODY()
	
	using FVariantType = TVariant<FUniqueNetIdPtr, UE::Online::FAccountId>;

	FUniqueNetIdWrapper() = default;
	virtual ~FUniqueNetIdWrapper() = default;

	// copy operators generated by compiler

	FUniqueNetIdWrapper(const FUniqueNetIdRef& InUniqueNetId)
	{
		Variant.Emplace<FUniqueNetIdPtr>(InUniqueNetId);
	}

	FUniqueNetIdWrapper(const FUniqueNetIdPtr& InUniqueNetId)
	{
		Variant.Emplace<FUniqueNetIdPtr>(InUniqueNetId);
	}

	FUniqueNetIdWrapper(const FVariantType& InVariant)
		: Variant(InVariant)
	{
	}

	FUniqueNetIdWrapper(const UE::Online::FAccountId& AccountId)
	{
		Variant.Emplace<UE::Online::FAccountId>(AccountId);
	}

	// temporarily restored implicit conversion from FUniqueNetId
	FUniqueNetIdWrapper(const FUniqueNetId& InUniqueNetId)
	{
		Variant.Emplace<FUniqueNetIdPtr>(InUniqueNetId.AsShared());
	}

	bool IsV1() const
	{
		return Variant.IsType<FUniqueNetIdPtr>();
	}

	FUniqueNetIdPtr GetV1() const
	{
		FUniqueNetIdPtr Result;
		if (ensure(IsV1()))
		{
			Result = Variant.Get<FUniqueNetIdPtr>();
		}
		return Result;
	}

	// Getter to be used only when the variant index has already been confirmed
	const FUniqueNetIdPtr& GetV1Unsafe() const
	{
		return Variant.Get<FUniqueNetIdPtr>();
	}

	bool IsV2() const
	{
		return Variant.IsType<UE::Online::FAccountId>();
	}

	UE::Online::FAccountId GetV2() const
	{
		UE::Online::FAccountId Result;
		if (ensure(IsV2()))
		{
			Result = Variant.Get<UE::Online::FAccountId>();
		}
		return Result;
	}

	// Getter to be used only when the variant index has already been confirmed
	const UE::Online::FAccountId& GetV2Unsafe() const
	{
		return Variant.Get<UE::Online::FAccountId>();
	}

	FName GetType() const
	{
		FName Result = NAME_None;
		if (IsValid() && ensure(IsV1()))
		{
			Result = GetV1Unsafe()->GetType();
		}
		return Result;
	}

	/** Convert this value to a string */
	COREONLINE_API FString ToString() const;

	/** Convert this value to a string with additional information */
	COREONLINE_API FString ToDebugString() const;

	/** Is the Variant wrapped in this object valid */
	bool IsValid() const
	{
		if (IsV1())
		{
			const FUniqueNetIdPtr& Ptr = GetV1Unsafe();
			return Ptr.IsValid() && Ptr->IsValid();
		}
		else
		{
			const UE::Online::FAccountId& AccountId = GetV2Unsafe();
			return AccountId.IsValid();
		}
	}

	/**
	 * Assign a unique id to this wrapper object
	 *
	 * @param InUniqueNetId id to associate
	 */
	virtual void SetUniqueNetId(const FUniqueNetIdPtr& InUniqueNetId)
	{
		Variant.Emplace<FUniqueNetIdPtr>(InUniqueNetId);
	}

	virtual void SetAccountId(const UE::Online::FAccountId& AccountId)
	{
		Variant.Emplace<UE::Online::FAccountId>(AccountId);
	}

	/** @return unique id associated with this wrapper object */
	FUniqueNetIdPtr GetUniqueNetId() const
	{
		return GetV1();
	}

	/**
	 * Dereference operator returns a reference to the FUniqueNetId
	 */
	const FUniqueNetId& operator*() const
	{
		return *GetV1();
	}

	/**
	 * Arrow operator returns a pointer to this FUniqueNetId
	 */
	const FUniqueNetId* operator->() const
	{
		return GetV1().Get();
	}

	/**
	* Friend function for using FUniqueNetIdWrapper as a hashable key
	*/
	friend inline uint32 GetTypeHash(const FUniqueNetIdWrapper& Value)
	{
		if (Value.IsValid())
		{
			if (Value.IsV1())
			{
				return GetTypeHash(*Value.GetV1Unsafe());
			}
			else
			{
				return GetTypeHash(Value.GetV2Unsafe());
			}
		}
		return INDEX_NONE;
	}

	friend FString LexToString(const FUniqueNetIdWrapper& Value)
	{
		return Value.ToString();
	}

	static FUniqueNetIdWrapper Invalid()
	{
		static FUniqueNetIdWrapper InvalidId(nullptr);
		return InvalidId;
	}

	friend bool operator==(const FUniqueNetIdWrapper& Lhs, const FUniqueNetIdWrapper& Rhs)
	{
		const bool bLhsValid = Lhs.IsValid();
		if (bLhsValid != Rhs.IsValid())
		{
			// Different validity
			return false;
		}
		if (!bLhsValid)
		{
			// Both invalid
			return true;
		}
		if (Lhs.Variant.GetIndex() != Rhs.Variant.GetIndex())
		{
			// Different variant
			return false;
		}

		if (Lhs.IsV1())
		{
			// Pointers can point to equivalent objects
			return *Lhs.GetV1Unsafe() == *Rhs.GetV1Unsafe();
		}
		else
		{
			return Lhs.GetV2Unsafe() == Rhs.GetV2Unsafe();
		}
	}

	friend bool operator!=(const FUniqueNetIdWrapper& Lhs, const FUniqueNetIdWrapper& Rhs)
	{
		return !(Lhs == Rhs);
	}

	friend bool operator==(const FUniqueNetIdWrapper& Lhs, const FUniqueNetId& Rhs)
	{
		const bool bLhsValid = Lhs.IsValid();
		if (bLhsValid != Rhs.IsValid())
		{
			// Different validity
			return false;
		}
		if (!bLhsValid)
		{
			// Both invalid
			return true;
		}
		if (!Lhs.IsV1())
		{
			// Different variant
			return false;
		}
		return *Lhs.GetV1Unsafe() == Rhs;
	}

	friend bool operator!=(const FUniqueNetIdWrapper& Lhs, const FUniqueNetId& Rhs)
	{
		return !(Lhs == Rhs);
	}

	friend bool operator==(const FUniqueNetId& Lhs, const FUniqueNetIdWrapper& Rhs)
	{
		return Rhs == Lhs;
	}

	friend bool operator!=(const FUniqueNetId& Lhs, const FUniqueNetIdWrapper& Rhs)
	{
		return Rhs != Lhs;
	}

	// comparison with nullptr (alternative to IsValid)
	friend bool operator==(const FUniqueNetIdWrapper& NetIdWrapper, TYPE_OF_NULLPTR)
	{
		return !NetIdWrapper.IsValid();
	}

	friend bool operator!=(const FUniqueNetIdWrapper& NetIdWrapper, TYPE_OF_NULLPTR)
	{
		return NetIdWrapper.IsValid();
	}

	friend bool operator==(TYPE_OF_NULLPTR, const FUniqueNetIdWrapper& NetIdWrapper)
	{
		return !NetIdWrapper.IsValid();
	}

	friend bool operator!=(TYPE_OF_NULLPTR, const FUniqueNetIdWrapper& NetIdWrapper)
	{
		return NetIdWrapper.IsValid();
	}

protected:

	// Actual unique id
	FVariantType Variant;
};

template <typename ValueType>
struct TUniqueNetIdMapKeyFuncs : public TDefaultMapKeyFuncs<FUniqueNetIdRef, ValueType, false>
{
	static FORCEINLINE FUniqueNetIdRef GetSetKey(TPair<FUniqueNetIdRef, ValueType> const& Element) { return Element.Key; }
	static FORCEINLINE uint32 GetKeyHash(FUniqueNetIdRef const& Key) { return GetTypeHash(*Key); }
	static FORCEINLINE bool Matches(FUniqueNetIdRef const& A, FUniqueNetIdRef const& B) { return (A == B) || (*A == *B); }
};

template <typename ValueType>
using TUniqueNetIdMap = TMap<FUniqueNetIdRef, ValueType, FDefaultSetAllocator, TUniqueNetIdMapKeyFuncs<ValueType>>;

struct FUniqueNetIdKeyFuncs : public DefaultKeyFuncs<FUniqueNetIdRef>
{
	static FORCEINLINE FUniqueNetIdRef GetSetKey(FUniqueNetIdRef const& Element) { return Element; }
	static FORCEINLINE uint32 GetKeyHash(FUniqueNetIdRef const& Key) { return GetTypeHash(*Key); }
	static FORCEINLINE bool Matches(FUniqueNetIdRef const& A, FUniqueNetIdRef const& B) { return (A == B) || (*A == *B); }
};

using FUniqueNetIdSet = TSet<FUniqueNetIdRef, FUniqueNetIdKeyFuncs>;
