// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FUniqueNetId;
struct FUniqueNetIdWrapper;

using FUniqueNetIdPtr = TSharedPtr<const FUniqueNetId>;
using FUniqueNetIdRef = TSharedRef<const FUniqueNetId>;
using FUniqueNetIdWeakPtr = TWeakPtr<const FUniqueNetId>;