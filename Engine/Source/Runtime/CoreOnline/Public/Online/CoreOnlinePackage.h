// Copyright Epic Games, Inc. All Rights Reserved.

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

#undef PACKAGE_SCOPE
#ifdef COREONLINE_PACKAGE
#define PACKAGE_SCOPE public
#else
#define PACKAGE_SCOPE protected
#endif
