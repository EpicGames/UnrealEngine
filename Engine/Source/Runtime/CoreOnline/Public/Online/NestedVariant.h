// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include <type_traits>

/**
 * This set of classes enables building nested TVariant structures using shared pointers to TArrays and TMaps. This can be extended
 * to other core container types or custom types as needed. If this proves useful beyond online use cases, this could move to Core.
 * 
 * This allows for usage such as:
 *
 * using FNestedVariant = TNestedVariant<FString, bool, int64, double, FString>; // The first FString is the TMap key type.
 * FNestedVariant::FMapRef Data = FNestedVariant::FMap::CreateVariant();
 *
 * Data->AddVariant(TEXT("MyString"), FString("1234-5678-9000"));
 * Data->AddVariant(TEXT("MyInt"), 42LL);
 * Data->AddVariant(TEXT("MyObject"), FNestedVariant::FMap::CreateVariant())->AddVariant(TEXT("MyNestedObjectDouble"), 1.0);
 * FNestedVariant::FArrayRef& Array = Data->AddVariant(TEXT("MyArray"), FNestedVariant::FArray::CreateVariant());
 *
 * Array->AddVariant(FNestedVariant::FMap::CreateVariant())->AddVariant(TEXT("MyNestedArrayObjectInt"), 100LL);
 * Array->AddVariant(FNestedVariant::FArray::CreateVariant())->AddVariant(1.23);
 * Array->AddVariant(4.0);
 * Array->AddVariant(true);
 *
 * FNestedVariant::FMapPtr NestedMap;
 * if (Array->GetVariant(0, NestedMap))
 * {
 *     // Found nested map in array.
 * }
 *
 * double ArrayDouble;
 * if (Array->GetVariant(2, ArrayDouble)) { ... }
 *
 * int64 MyInt;
 * if (Data->GetVariant(TEXT("MyInt"), MyInt)) { ... }
 *
 * FNestedVariant::FArrayPtr NestedArray;
 * if (Data->GetVariant(TEXT("MyArray"), NestedArray))
 * {
 *     bool Value;
 *     if (NestedArray->GetVariant(3, Value)) { ... }
 * }
 * 
 * OnlineUtils.h has ToLogString() overrides to allow for: ToLogString(Data);
 * Which would output: {MyString:1234-5678-9000, MyInt:42, MyObject:{MyNestedObjectDouble:1.00}, MyArray:[{MyNestedArrayObjectInt:100}, [1.23], 4.00, true]}
 */


/** TArray that holds a TVariant with helper functions for working with variant types. */
template<typename... ValueTypes>
class TVariantArray : public TArray<TVariant<ValueTypes...>>, public TSharedFromThis<TVariantArray<ValueTypes...>>
{
public:
	using VariantType = TVariant<ValueTypes...>;
	using ArrayType = TArray<VariantType>;

	static TSharedRef<TVariantArray<ValueTypes...>> CreateVariant()
	{
		return MakeShared<TVariantArray<ValueTypes...>>();
	}

	template<typename ValueType, typename VariantValueType = std::remove_cv_t<std::remove_reference_t<ValueType>>>
	VariantValueType& AddVariant(ValueType&& Value)
	{
		return ArrayType::Emplace_GetRef(VariantType(TInPlaceType<VariantValueType>(), Forward<ValueType>(Value))).template Get<VariantValueType>();
	}

	template<typename ValueType>
	void AddVariantArray(const TArray<ValueType>& Array)
	{
		for (const ValueType& Value : Array)
		{
			AddVariant(Value);
		}
	}

	template<typename ValueType>
	bool GetVariant(ArrayType::SizeType Index, TSharedPtr<ValueType>& OutValue) const
	{
		if (Index >= 0 && Index < ArrayType::Num())
		{
			if (ArrayType::operator[](Index).template IsType<TSharedRef<ValueType>>())
			{
				OutValue = ArrayType::operator[](Index).template Get<TSharedRef<ValueType>>();
				return true;
			}
		}

		return false;
	}

	template<typename ValueType>
	bool GetVariant(ArrayType::SizeType Index, ValueType& OutValue) const
	{
		if (Index >= 0 && Index < ArrayType::Num())
		{
			if (ArrayType::operator[](Index).template IsType<ValueType>())
			{
				OutValue = ArrayType::operator[](Index).template Get<ValueType>();
				return true;
			}
		}

		return false;
	}

	template<typename ValueType>
	bool GetVariantArray(TArray<ValueType>& OutValue) const
	{
		OutValue.Reserve(ArrayType::Num());
		for (int32 Index = 0; Index < ArrayType::Num(); Index++)
		{
			if (ArrayType::operator[](Index).template IsType<ValueType>())
			{
				OutValue.Add(ArrayType::operator[](Index).template Get<ValueType>());
			}
		}

		return ArrayType::Num() == OutValue.Num();
	}
};

/** TMap that holds a TVariant with helper functions for working with variant types. */
template<typename KeyType, typename... ValueTypes>
class TVariantMap : public TMap<KeyType, TVariant<ValueTypes...>>, public TSharedFromThis<TVariantMap<KeyType, ValueTypes...>>
{
public:
	using VariantType = TVariant<ValueTypes...>;
	using MapType = TMap<KeyType, VariantType>;

	static TSharedRef<TVariantMap<KeyType, ValueTypes...>> CreateVariant()
	{
		return MakeShared<TVariantMap<KeyType, ValueTypes...>>();
	}

	template<typename ValueType, typename AddKeyType, typename VariantValueType = std::remove_cv_t<std::remove_reference_t<ValueType>>>
	VariantValueType& AddVariant(AddKeyType&& Key, ValueType&& Value)
	{
		return MapType::Emplace(Forward<AddKeyType>(Key), VariantType(TInPlaceType<VariantValueType>(), Forward<ValueType>(Value))).template Get<VariantValueType>();
	}

	template<typename ValueType>
	void AddVariantMap(const TMap<KeyType, ValueType>& Map)
	{
		for (const TPair<KeyType, ValueType>& Pair : Map)
		{
			AddVariant(Pair.Key, Pair.Value);
		}
	}

	template<typename ValueType, typename GetKeyType>
	bool GetVariant(const GetKeyType& Key, TSharedPtr<ValueType>& OutValue) const
	{
		if (const VariantType* Attribute = MapType::Find(Key))
		{
			if (Attribute->template IsType<TSharedRef<ValueType>>())
			{
				OutValue = Attribute->template Get<TSharedRef<ValueType>>();
				return true;
			}
		}

		return false;
	}

	template<typename ValueType, typename GetKeyType>
	bool GetVariant(const GetKeyType& Key, ValueType& OutValue) const
	{
		if (const VariantType* Attribute = MapType::Find(Key))
		{
			if (Attribute->template IsType<ValueType>())
			{
				OutValue = Attribute->template Get<ValueType>();
				return true;
			}
		}

		return false;
	}

	template<typename ValueType>
	bool GetVariantMap(TMap<KeyType, ValueType>& OutValue) const
	{
		OutValue.Reserve(MapType::Num());
		for (const TPair<KeyType, VariantType>& Pair : MapType::Pairs)
		{
			if (Pair.Value.template IsType<ValueType>())
			{
				OutValue.Add(Pair.Key, Pair.Value.template Get<ValueType>());
			}
		}

		return MapType::Num() == OutValue.Num();
	}
};

// Classes for working with nested TVariants that also have TArray and TMap references holding TVariants of the same types.
// This is just one possible nesting structure. Other custom structures can be built for various use cases (no arrays, arrays with fixed types, etc).
// First declare types to allow for nesting.

template<typename KeyType, typename... ValueTypes> class TNestedVariantArray;
template<typename KeyType, typename... ValueTypes> using TNestedVariantArrayRef = TSharedRef<TNestedVariantArray<KeyType, ValueTypes...>>;
template<typename KeyType, typename... ValueTypes> using TNestedVariantArrayPtr = TSharedPtr<TNestedVariantArray<KeyType, ValueTypes...>>;

template<typename KeyType, typename... ValueTypes> class TNestedVariantMap;
template<typename KeyType, typename... ValueTypes> using TNestedVariantMapRef = TSharedRef<TNestedVariantMap<KeyType, ValueTypes...>>;
template<typename KeyType, typename... ValueTypes> using TNestedVariantMapPtr = TSharedPtr<TNestedVariantMap<KeyType, ValueTypes...>>;

template<typename KeyType, typename... ValueTypes> using FNestedVariantValue = TVariant<ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef<KeyType, ValueTypes...>>;


// Common methods between TNestedVariantArray and TNestedVariantMap.
namespace UE::Online::NestedVariant
{
	template<typename VisitType>
	using TGuardType = TSet<VisitType>;

	// Single void* methods for use with AppendArray and AppendMap.
	using FSingleType = const void*;
	using FSingleVisit = TGuardType<FSingleType>;

	[[nodiscard]] inline FSingleType MakeID(const void* InID)
	{
		return InID;
	}

	[[nodiscard]] inline bool ContainsID(const FSingleVisit& VisitSet, FSingleType ID)
	{
		return VisitSet.Contains(ID);
	}

	[[nodiscard]] inline FString ToString(FSingleType InID)
	{
		return FString::Printf(TEXT("%p"), InID);
	}
	// ~Single void* methods.

	// Dual void* methods for use with Array and Map operator==.
	using FDualType = TPair<const void*, const void*>;
	using FDualVisit = TGuardType<FDualType>;

	[[nodiscard]] inline FDualType MakeID(const void* Left, const void* Right)
	{
		return MakeTuple(Left, Right);
	}

	[[nodiscard]] inline bool ContainsID(const FDualVisit& VisitSet, FDualType& ID)
	{
		// Check Left/Right and Right/Left variants.
		return VisitSet.Contains(ID) || VisitSet.Contains(FDualType{ ID.Value, ID.Key });
	}

	[[nodiscard]] inline FString ToString(FDualType InPair)
	{
		return FString::Printf(TEXT("%p, %p"), InPair.Key, InPair.Value);
	}
	// ~Dual void* methods.

	// Recursion protection for nested composites.
	template<typename VisitType, int32 MaxDepth>
	struct TRecursionScope
	{
		TGuardType<VisitType>& Visited;
		VisitType ID;
		bool bSuccess = false;

		TRecursionScope() = delete;

		TRecursionScope(TGuardType<VisitType>& InVisited, const void* InID) : Visited(InVisited), ID(MakeID(InID))
		{
			Enter();
		}

		TRecursionScope(TGuardType<VisitType>& InVisited, const void* InLeft, const void* InRight) : Visited(InVisited), ID(MakeID(InLeft, InRight))
		{
			Enter();
		}

		~TRecursionScope()
		{
			// Pop the scope if we didn't fail our checks.
			if (bSuccess)
			{
				Visited.Remove(ID);
			}
		}

		[[nodiscard]] explicit operator bool() const { return bSuccess; }

		[[nodiscard]] FString GetError() const
		{
			if (bSuccess)
			{
				return FString();
			}
			else
			{
				if (Visited.Num() >= MaxDepth)
				{
					return FString::Printf(TEXT("Max Depth Reached: %d!"), MaxDepth);
				}
				else
				{
					return FString::Printf(TEXT("Duplicate scope: %s!"), *ToString(ID));
				}
			}
		}

	private:
		void Enter()
		{
			// Check max recursion depth.
			if (Visited.Num() >= MaxDepth)
			{
				return;
			}

			// Check to see if we have already visited this node.
			if (ContainsID(Visited, ID))
			{
				return;
			}

			// Push/track this scope.
			bSuccess = Visited.Add(ID).IsValidId();
		}
	};

	// Comparison functions for use with operator==.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes, typename T>
	[[nodiscard]] bool Compare(const T& Lhs, const T& Rhs, FDualVisit& /* Unused */)
	{
		return Lhs == Rhs;
	}

	// Forward declaration for use in various Compare functions.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool CompareVariants(const FNestedVariantValue<KeyType, ValueTypes...>& Lhs, const FNestedVariantValue<KeyType, ValueTypes...>& Rhs, FDualVisit& VisitedSet);

	// TNestedVariantArray comparison.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool Compare(const TNestedVariantArray<KeyType, ValueTypes...>& Lhs, const TNestedVariantArray<KeyType, ValueTypes...>& Rhs, FDualVisit& Visited)
	{
		using ThisType = TVariantArray<ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef<KeyType, ValueTypes...>>;
		const ThisType& LeftTyped = static_cast<const ThisType&>(Lhs);
		const ThisType& RightTyped = static_cast<const ThisType&>(Rhs);

		// Compare addresses.
		if (&Lhs == &Rhs)
		{
			return true;
		}

		// If the arrays have differing numbers of variants then they are not equal.
		if (LeftTyped.Num() != RightTyped.Num())
		{
			return false;
		}

		// Check recursion scope/depth.
		const TRecursionScope<FDualType, MaxDepth> Scope(Visited, &Lhs, &Rhs);
		if (!Scope)
		{
			return false;
		}

		for (int32 Index = 0; Index < LeftTyped.Num(); ++Index)
		{
			// Order-dependent equals.
			if (!CompareVariants<MaxDepth, KeyType, ValueTypes...>(LeftTyped[Index], RightTyped[Index], Visited))
			{
				return false;
			}
		}

		return true;
	}

	// Dereference TNestedVariantArrayRef to TNestedVariantArray.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool Compare(const TNestedVariantArrayRef<KeyType, ValueTypes...>& Lhs, const TNestedVariantArrayRef<KeyType, ValueTypes...>& Rhs, FDualVisit& Visited)
	{
		return Compare<MaxDepth, KeyType, ValueTypes...>(*Lhs, *Rhs, Visited);
	}

	// TNestedVariantMap comparison.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool Compare(const TNestedVariantMap<KeyType, ValueTypes...>& Lhs, const TNestedVariantMap<KeyType, ValueTypes...>& Rhs, FDualVisit& Visited)
	{
		using ThisType = TVariantMap<KeyType, ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef<KeyType, ValueTypes...>>;
		using MapType = typename ThisType::MapType;
		using VariantType = typename ThisType::VariantType;

		const MapType& LeftTyped = static_cast<const MapType&>(Lhs);
		const MapType& RightTyped = static_cast<const MapType&>(Rhs);

		// Compare addresses.
		if (&Lhs == &Rhs)
		{
			return true;
		}

		// If the maps have differing numbers of variants then they are not equal.
		if (LeftTyped.Num() != RightTyped.Num())
		{
			return false;
		}

		// Check recursion scope/depth.
		const TRecursionScope<FDualType, MaxDepth> Scope(Visited, &Lhs, &Rhs);
		if (!Scope)
		{
			return false;
		}

		// Order-independent comparison (constant-time TMap lookup).
		for (const TPair<KeyType, VariantType>& Pair : LeftTyped)
		{
			// Find the left map's variants in the right map.
			const VariantType* RightVal = RightTyped.Find(Pair.Key);
			if (!RightVal)
			{
				return false;
			}

			// Found it, now compare.
			if (!CompareVariants<MaxDepth, KeyType, ValueTypes...>(Pair.Value, *RightVal, Visited))
			{
				return false;
			}
		}

		return true;
	}

	// Dereference TNestedVariantMapRef to TNestedVariantMap.
	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool Compare(const TNestedVariantMapRef<KeyType, ValueTypes...>& Lhs, const TNestedVariantMapRef<KeyType, ValueTypes...>& Rhs, FDualVisit& Visited)
	{
		return Compare<MaxDepth, KeyType, ValueTypes...>(*Lhs, *Rhs, Visited);
	}

	template<int32 MaxDepth, typename KeyType, typename... ValueTypes>
	[[nodiscard]] bool CompareVariants(const FNestedVariantValue<KeyType, ValueTypes...>& Lhs, const FNestedVariantValue<KeyType, ValueTypes...>& Rhs, FDualVisit& VisitedSet)
	{
		return ::Visit([&VisitedSet](auto const& LVisited, auto const& RVisited)
		{
			using LeftDecayed = std::decay_t<decltype(LVisited)>;
			using RightDecayed = std::decay_t<decltype(RVisited)>;
			if constexpr (!std::is_same_v<LeftDecayed, RightDecayed>)
			{
				return false;
			}
			else
			{
				return Compare<MaxDepth, KeyType, ValueTypes...>(LVisited, RVisited, VisitedSet);
			}
		}, Lhs, Rhs);
	}

	// String appending methods for use with ToDebugString().
	inline void AppendIndent(FString& OutString, int32 TabDepth)
	{
		for (int32 i = 0; i < TabDepth; ++i)
		{
			OutString.AppendChar(TEXT('\t'));
		}
	}

	// Forward declarations used in AppendVariant.
	template<int32 MaxDepth, int32 TabOffset, typename KeyType, typename... ValueTypes>
	void AppendArray(const TNestedVariantArray<KeyType, ValueTypes...>& Array, FString& OutString, FSingleVisit& Visited);

	template<int32 MaxDepth, int32 TabOffset, typename KeyType, typename... ValueTypes>
	void AppendMap(const TNestedVariantMap<KeyType, ValueTypes...>& Map, FString& OutString, FSingleVisit& Visited);

	template<typename T>
	inline void AppendValue(FString& OutString, const T& V)
	{
		OutString += ::LexToString(V);
	}

	inline void AppendValue(FString& OutString, const FString& S)
	{
		OutString += S;
	}

	inline void AppendValue(FString& OutString, double Double, int32 Precision)
	{
		OutString += FString::Printf(TEXT("%.*f"), Precision, Double);
	}

	template<int32 MaxDepth, int32 TabOffset, bool bAfterKey, typename KeyType, typename... ValueTypes>
	void AppendVariant(const FNestedVariantValue<KeyType, ValueTypes...>& Variant, FString& OutString, FSingleVisit& VisitedSet)
	{
		::Visit([&VisitedSet, &OutString](auto const& Visited)
		{
			using DecayedType = std::decay_t<decltype(Visited)>;
			if constexpr (std::is_same_v<DecayedType, TNestedVariantArrayRef<KeyType, ValueTypes...>>)
			{
				// If we printed a map with a Key: in the log.
				if constexpr (bAfterKey)
				{
					OutString.AppendChar(TEXT('\n'));
				}

				AppendArray<MaxDepth, TabOffset, KeyType, ValueTypes...>(*Visited, OutString, VisitedSet);
			}
			else if constexpr (std::is_same_v<DecayedType, TNestedVariantMapRef<KeyType, ValueTypes...>>)
			{
				// If we printed a map with a Key: in the log.
				if constexpr (bAfterKey)
				{
					OutString.AppendChar(TEXT('\n'));
				}

				AppendMap<MaxDepth, TabOffset, KeyType, ValueTypes...>(*Visited, OutString, VisitedSet);
			}
			else if constexpr (std::is_floating_point_v<DecayedType>)
			{
				// If we're not after a key then we're on our own line and need indentation.
				if constexpr (!bAfterKey)
				{
					AppendIndent(OutString, TabOffset + VisitedSet.Num());
				}

				AppendValue(OutString, static_cast<double>(Visited), 2);
				OutString.AppendChar(TEXT('\n'));
			}
			else
			{
				// If we're not after a key then we're on our own line and need indentation.
				if constexpr (!bAfterKey)
				{
					AppendIndent(OutString, TabOffset + VisitedSet.Num());
				}

				AppendValue(OutString, Visited);
				OutString.AppendChar(TEXT('\n'));
			}
		}, Variant);
	}

	template<int32 MaxDepth, int32 TabOffset, typename KeyType, typename... ValueTypes>
	void AppendArray(const TNestedVariantArray<KeyType, ValueTypes...>& Array, FString& OutString, FSingleVisit& Visited)
	{
		const int32 TabDepth = Visited.Num() + TabOffset;

		// Check recursion scope/depth.
		const TRecursionScope<FSingleType, MaxDepth> Scope(Visited, &Array);
		if (!Scope)
		{
			AppendIndent(OutString, TabDepth);
			OutString += Scope.GetError();
			OutString.AppendChar(TEXT('\n'));
			return;
		}

		AppendIndent(OutString, TabDepth);
		OutString += TEXT("[\n");
		
		const int32 ArrayNum = Array.Num();
		if (ArrayNum > 0)
		{
			// Arrays do not have Key/Value pairs.
			constexpr bool bAfterKey = false;

			for (int32 i = 0; i < ArrayNum; ++i)
			{
				AppendVariant<MaxDepth, TabOffset, bAfterKey, KeyType, ValueTypes...>(Array[i], OutString, Visited);
			}
		}

		AppendIndent(OutString, TabDepth);
		OutString += TEXT("]\n");
	}

	template<int32 MaxDepth, int32 TabOffset, typename KeyType, typename... ValueTypes>
	void AppendMap(const TNestedVariantMap<KeyType, ValueTypes...>& Map, FString& OutString, FSingleVisit& Visited)
	{
		using ThisType = TVariantMap<KeyType, ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef <KeyType, ValueTypes...>>;
		using MapType = typename ThisType::MapType;
		using VariantType = FNestedVariantValue<KeyType, ValueTypes...>;

		const MapType& MapBase = static_cast<const MapType&>(Map);

		const int32 TabDepth = Visited.Num() + TabOffset;

		// Check recursion scope/depth.
		const TRecursionScope<FSingleType, MaxDepth> Scope(Visited, &Map);
		if (!Scope)
		{
			AppendIndent(OutString, TabDepth);
			OutString += Scope.GetError();
			OutString.AppendChar(TEXT('\n'));
			return;
		}

		AppendIndent(OutString, TabDepth);
		OutString += TEXT("{\n");

		// All variants appended to the string will be after their associated Key.
		constexpr bool bAfterKey = true;

		for (const TPair<KeyType, VariantType>& Pair : MapBase)
		{
			AppendIndent(OutString, TabDepth + 1);
			OutString += ::LexToString(Pair.Key);
			OutString += TEXT(": ");

			AppendVariant<MaxDepth, TabOffset, bAfterKey, KeyType, ValueTypes...>(Pair.Value, OutString, Visited);
		}

		AppendIndent(OutString, TabDepth);
		OutString += TEXT("}\n");
	}
} // namespace UE::Online::NestedVariant


/** Nested TVariantArray that can hold all provided types plus nested array and map references. */
template<typename KeyType, typename... ValueTypes>
class TNestedVariantArray : public TVariantArray<ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef<KeyType, ValueTypes...>>
{
public:
	static TNestedVariantArrayRef<KeyType, ValueTypes...> CreateVariant() {
		return MakeShared<TNestedVariantArray<KeyType, ValueTypes...>>();
	}

	bool operator==(const TNestedVariantArray& Other) const
	{
		constexpr int32 MaxDepth = 64;
		UE::Online::NestedVariant::FDualVisit Visited;
		return UE::Online::NestedVariant::Compare<MaxDepth>(*this, Other, Visited);
	}

	bool operator!=(const TNestedVariantArray& Other) const
	{
		return !(*this == Other);
	}

	template<int32 MaxDepth = 32, int32 TabOffset = 0>
	void ToDebugString(FString& OutString) const
	{
		OutString.Reset(256);

		UE::Online::NestedVariant::FSingleVisit Visited;
		UE::Online::NestedVariant::AppendArray<MaxDepth, TabOffset>(*this, OutString, Visited);

		// Strip the final newline.
		OutString.RemoveFromEnd(TEXT("\n"));
	}
};

/** Nested TVariantMap that can hold all provided types plus nested array and map references. */
template<typename KeyType, typename... ValueTypes>
class TNestedVariantMap : public TVariantMap<KeyType, ValueTypes..., TNestedVariantArrayRef<KeyType, ValueTypes...>, TNestedVariantMapRef<KeyType, ValueTypes...>>
{
public:
	static TNestedVariantMapRef<KeyType, ValueTypes...> CreateVariant() {
		return MakeShared<TNestedVariantMap<KeyType, ValueTypes...>>();
	}

	bool operator==(const TNestedVariantMap& Other) const
	{
		constexpr int32 MaxDepth = 64;
		UE::Online::NestedVariant::FDualVisit Visited;
		return UE::Online::NestedVariant::Compare<MaxDepth>(*this, Other, Visited);
	}

	bool operator!=(const TNestedVariantMap& Other) const
	{
		return !(*this == Other);
	}

	template<int32 MaxDepth = 32, int32 TabOffset = 0>
	void ToDebugString(FString& OutString) const
	{
		OutString.Reset(256);

		UE::Online::NestedVariant::FSingleVisit Visited;
		UE::Online::NestedVariant::AppendMap<MaxDepth, TabOffset>(*this, OutString, Visited);

		// Strip the final newline.
		OutString.RemoveFromEnd(TEXT("\n"));
	}
};

/** Convenience class to declare all nested types at once. */
template<typename KeyType, typename... ValueTypes>
class TNestedVariant
{
public:
	using FArray = TNestedVariantArray<KeyType, ValueTypes...>;
	using FArrayRef = TNestedVariantArrayRef<KeyType, ValueTypes...>;
	using FArrayPtr = TNestedVariantArrayPtr<KeyType, ValueTypes...>;

	using FMap = TNestedVariantMap<KeyType, ValueTypes...>;
	using FMapRef = TNestedVariantMapRef<KeyType, ValueTypes...>;
	using FMapPtr = TNestedVariantMapPtr<KeyType, ValueTypes...>;

	using FValue = FNestedVariantValue<KeyType, ValueTypes...>;
};

