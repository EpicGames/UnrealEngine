// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Online/NestedVariant.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"

// TODO: Update the JSON module to handle nested variants of any type directly so this file is not needed.

using FNestedVariantJson = TNestedVariant<FString, bool, int64, double, FString>;
class FJsonObject;
class FJsonValue;

COREONLINE_API FString NestedVariantToJson(const FNestedVariantJson::FMapPtr& Map);
COREONLINE_API TSharedRef<FJsonObject> NestedVariantToJsonObject(const FNestedVariantJson::FMapRef& Map);
COREONLINE_API TSharedRef<FJsonObject> NestedVariantToJsonObject(const FNestedVariantJson::FMap& Map);

COREONLINE_API void NestedVariantFromJson(const char* InJson, FNestedVariantJson::FMapRef& Map);
COREONLINE_API void NestedVariantFromJson(const char* InJson, FNestedVariantJson::FMap& Map);
COREONLINE_API void NestedVariantFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, FNestedVariantJson::FMapRef& Map);
COREONLINE_API void NestedVariantFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, FNestedVariantJson::FMap& Map);

COREONLINE_API void LexFromString(FNestedVariantJson::FMap& OutValue, const TCHAR* InString);

struct FJsonSerializerPolicy_NestedVariant
{
	using FValue = FNestedVariantJson::FValue;
	using FArrayOfValues = FNestedVariantJson::FArrayPtr;
	using FMapOfValues = FNestedVariantJson::FMapPtr;

	struct FGlobalState
	{
	};

	struct FStackState
	{
		FStackState() = default;
		explicit FStackState(const FGlobalState&) {}

		EJson Type;
		FString Identifier;
		FNestedVariantJson::FArrayPtr Array;
		FNestedVariantJson::FMapPtr Object;
	};

	struct FElement
	{
		FElement( const FValue& InValue )
			: Identifier()
			, Value(InValue)
		{ }

		FElement( const FMapOfValues& Object )
			: Identifier()
		{
			Value.Set<FNestedVariantJson::FMapRef>(StaticCastSharedRef<FNestedVariantJson::FMap>(Object->AsShared()));
		}

		FElement( const FArrayOfValues& Array )
			: Identifier()
		{
			Value.Set<FNestedVariantJson::FArrayRef>(StaticCastSharedRef<FNestedVariantJson::FArray>(Array->AsShared()));
		}

		FElement( const FString& InIdentifier, const FValue& InValue )
			: Identifier( InIdentifier )
			, Value( InValue )
			, bIsKeyValuePair( true )
		{ }

		FString Identifier;
		FValue Value;
		bool bHasBeenProcessed = false;
		bool bIsKeyValuePair = false;
	};

	COREONLINE_API static bool GetValueFromState(const FStackState& State, FValue& OutValue);
	COREONLINE_API static bool GetValueFromState(const FStackState& State, FArrayOfValues& OutArray);
	COREONLINE_API static bool GetValueFromState(const FStackState& State, FMapOfValues& OutMap);
	COREONLINE_API static void ResetValue(FValue& OutValue);
	COREONLINE_API static void ReadObjectStart(FStackState& State);
	COREONLINE_API static void ReadObjectEnd(FStackState& State, FValue& OutValue);
	COREONLINE_API static void ReadArrayStart(FStackState& State);
	COREONLINE_API static void ReadArrayEnd(FStackState& State, FValue& OutValue);

	template<class CharType>
	static void ReadBoolean(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		OutValue.template Emplace<bool>(Reader.GetValueAsBoolean());
	}

	template<class CharType>
	static void ReadString(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		OutValue.template Emplace<FString>(Reader.StealInternalValueAsString());
	}

	template<class CharType>
	static void ReadNumberAsString(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		OutValue.template Emplace<FString>(Reader.GetValueAsNumberString());
	}

	template<class CharType>
	static bool IsNumberStringInteger(const CharType* Str)
	{
		if (*Str == '-' || *Str == '+')
		{
			Str++;
		}

		while (*Str != '\0')
		{
			if (*Str == '.')
			{
				return false;
			}
			else if (!FChar::IsDigit(*Str))
			{
				return false;
			}
			
			++Str;
		}

		return true;
	}

	template<class CharType>
	static void ReadNumber(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		const typename TJsonReader<CharType>::StoredStringType& NumberString = Reader.GetValueAsNumberString();
		if (IsNumberStringInteger(*NumberString))
		{
			OutValue.template Emplace<long long>(Reader.GetValueAsNumber());
		}
		else
		{
			OutValue.template Emplace<double>(Reader.GetValueAsNumber());
		}
	}

	static void ReadNull(FValue& OutValue);
	static void AddValueToObject(FStackState& State, const FString& Identifier, FValue& NewValue);
	static void AddValueToArray(FStackState& State, FValue& NewValue);

	template<class CharType, class PrintPolicy>
	static bool SerializeIfBool(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value.IsType<bool>())
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value.Get<bool>());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value.Get<bool>());
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfNumber(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if (Element->Value.IsType<long long>())
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value.Get<long long>());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value.Get<long long>());
			}

			return true;
		}
		else if(Element->Value.IsType<double>())
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value.Get<double>());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value.Get<double>());
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfString(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value.IsType<FString>())
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value.Get<FString>());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value.Get<FString>());
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfNull(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{		
		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfArray(TArray<TSharedRef<FElement>>& ElementStack, TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value.IsType<FNestedVariantJson::FArrayRef>())
		{
			if (Element->bHasBeenProcessed)
			{
				Writer.WriteArrayEnd();
			}
			else
			{
				Element->bHasBeenProcessed = true;
				ElementStack.Push(Element);

				if (bWriteValueOnly)
				{
					Writer.WriteArrayStart();
				}
				else
				{
					Writer.WriteArrayStart(Element->Identifier);
				}

				FArrayOfValues Values = StaticCastSharedPtr<FNestedVariantJson::FArray>(Element->Value.Get<FNestedVariantJson::FArrayRef>().ToSharedPtr());

				for (int Index = Values->Num() - 1; Index >= 0; --Index)
				{
					ElementStack.Push(MakeShared<FElement>((*Values)[Index]));
				}
			}

			return true;
		}
		
		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfObject(TArray<TSharedRef<FElement>>& ElementStack, TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if (Element->Value.IsType<FNestedVariantJson::FMapRef>())
		{
			if (Element->bHasBeenProcessed)
			{
				Writer.WriteObjectEnd();
			}
			else
			{
				Element->bHasBeenProcessed = true;
				ElementStack.Push(Element);

				if (bWriteValueOnly)
				{
					Writer.WriteObjectStart();
				}
				else
				{
					Writer.WriteObjectStart(Element->Identifier);
				}

				TArray<FString> Keys; 
				TArray<FValue> Values;
				FMapOfValues ElementMap = StaticCastSharedPtr<FNestedVariantJson::FMap>(Element->Value.Get<FNestedVariantJson::FMapRef>().ToSharedPtr());
				for (const FNestedVariantJson::FMap::ElementType& Entry : *ElementMap)
				{
					Keys.Add(Entry.Key);
					
					Values.Add(Entry.Value);
				}

				check(Keys.Num() == Values.Num());

				for (int Index = Values.Num() - 1; Index >= 0; --Index)
				{
					ElementStack.Push(MakeShared<FElement>(Keys[Index], Values[Index]));
				}
			}

			return true;
		}
		
		return false;
	}
};